// Copyright Epic Games, Inc. All Rights Reserved.

#include "/Engine/Public/Platform.ush"


float4x4	MeshToPFMMatrix;

struct InputVS
{
	float4 Position : ATTRIBUTE0;
	float2 UV       : ATTRIBUTE1;
};
struct OutputVS
{
	float4	Position    : SV_POSITION;
	float3	PFMPosition : NORMAL;
};
struct OutputPS
{
	float4 PFMPosition  : SV_Target0;
};

OutputVS PFMExporterUV_VS(InputVS IN)
{
	OutputVS Out;

	// Texel from UV channel (0..1) mapped to screen (-1..1)
	float2 Pos = IN.UV.xy;
	Pos.xy = Pos.xy * 2 - 1.0f;
	Pos.xy *= float2(1, -1);
	Out.Position = float4(Pos.xy, 0, 1);

	// Value is world-space mesh point, transformed to origin
	Out.PFMPosition = mul(IN.Position, MeshToPFMMatrix).xyz;
	return Out;
}

OutputPS PFMExporterPassthrough_PS(OutputVS IN)
{
	OutputPS Out;
	Out.PFMPosition = float4(IN.PFMPosition,1);
	return Out;
}
