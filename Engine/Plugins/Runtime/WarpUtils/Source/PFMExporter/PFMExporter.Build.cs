// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;
using System.IO;

public class PFMExporter : ModuleRules
{
	public PFMExporter(ReadOnlyTargetRules ROTargetRules) : base(ROTargetRules)
	{
		PublicDefinitions.Add("PFMExporter_STATIC");

		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
            });

		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
                "Projects",
				"RenderCore",
				"RHI"
			}
		);


		if (Target.bBuildEditor == true)
		{
			PrivateDependencyModuleNames.Add("EditorFramework");
			PrivateDependencyModuleNames.Add("UnrealEd");
		}
	}
}
