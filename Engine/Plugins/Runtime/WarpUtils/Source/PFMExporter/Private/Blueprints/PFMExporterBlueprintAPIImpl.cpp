// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprints/PFMExporterBlueprintAPIImpl.h"
#include "IPFMExporter.h"

#include "Engine/StaticMesh.h"
#include "UObject/Package.h"
#include "Components/StaticMeshComponent.h"
#include "Components/SceneComponent.h"

bool UPFMExporterAPIImpl::ExportPFM(
	UStaticMeshComponent* SrcMeshComponent,
	USceneComponent*      PFMOrigin,
	int Width,
	int Height,
	const FString& FileName
)
{
	if (SrcMeshComponent==nullptr)
	{
		//! handle error
		return false;
	}
	
	// Use attached root as origin 
	//! All geometry links scaled, etc at this point. Finally we send MeshToOrigin matrix to shader (transfrom from mesh space to Origin)
	USceneComponent* OriginComp = (PFMOrigin) ? PFMOrigin : SrcMeshComponent->GetAttachParent();
	
	const FTransform& OriginToWorldTransform = (OriginComp ? OriginComp->GetComponentTransform() : FTransform::Identity);
	
	FTransform MeshToWorldTransform = SrcMeshComponent->GetComponentTransform();

	FMatrix WorldToOrigin = OriginToWorldTransform.ToInverseMatrixWithScale();
	FMatrix MeshToWorld   = MeshToWorldTransform.ToMatrixWithScale();

	FMatrix MeshToOrigin = MeshToWorld * WorldToOrigin;

	UStaticMesh* StaticMesh = SrcMeshComponent->GetStaticMesh();
	
	if(StaticMesh==nullptr)
	{ 
		//! Handle error
		return false;
	}

	if (IPFMExporter::Get().ExportPFM(&StaticMesh->GetLODForExport(0), MeshToOrigin, Width, Height, FileName))
	{
		return true;
	}

	//! handle error
	return false;
}