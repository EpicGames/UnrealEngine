// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprints/PFMExporterBlueprintLib.h"
#include "Blueprints/PFMExporterBlueprintAPIImpl.h"
#include "UObject/Package.h"


UPFMExporterBlueprintLib::UPFMExporterBlueprintLib(class FObjectInitializer const & ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UPFMExporterBlueprintLib::GetAPI(TScriptInterface<IPFMExporterBlueprintAPI>& OutAPI)
{
	static UPFMExporterAPIImpl* Obj = NewObject<UPFMExporterAPIImpl>(GetTransientPackage(), NAME_None, RF_MarkAsRootSet);
	OutAPI = Obj;
}
