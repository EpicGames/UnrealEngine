// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"

// Plugin-wide log categories
#if UE_BUILD_SHIPPING
DECLARE_LOG_CATEGORY_EXTERN(LogPFMExporter, Warning, Warning);
#else
DECLARE_LOG_CATEGORY_EXTERN(LogPFMExporter, Log, All);
#endif
