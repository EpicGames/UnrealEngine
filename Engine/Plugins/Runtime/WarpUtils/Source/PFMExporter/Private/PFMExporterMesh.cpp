// Copyright Epic Games, Inc. All Rights Reserved.

#include "PFMExporterMesh.h"
#include "PFMExporterLog.h"

#include "Engine/Engine.h"
#include "Engine/RendererSettings.h"

#include "Stats/Stats.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/Paths.h"
#include "Misc/FileHelper.h"

#include "RendererInterface.h"
#include "CommonRenderResources.h"
#include "RenderResource.h"
#include "SceneTypes.h"

static const EPixelFormat PFMTextureFormat = PF_A32B32G32R32F;
static const uint32       PFMPixelSize = 4 * 4;

FPFMExporterMesh::FPFMExporterMesh(int Width, int Height)
	: DimWidth(Width)
	, DimHeight(Height)
{	
}

FPFMExporterMesh::~FPFMExporterMesh()
{	
}

bool FPFMExporterMesh::SaveToFile(const FString& FileName)
{
	TUniquePtr<FArchive> FileWriter(IFileManager::Get().CreateFileWriter(*FileName));
	if (!FileWriter.IsValid())
	{
		UE_LOGF(LogPFMExporter, Error, "Couldn't create a file writer %ls", *FileName);
		return false;
	}

	const FString Header = FString::Printf(TEXT("PF%c%d %d%c-1%c"), 0x0A, DimWidth, DimHeight, 0x0A, 0x0A);
	FileWriter->Serialize(TCHAR_TO_ANSI(*Header), Header.Len());
	FileWriter->Serialize((void*)PFMPoints.GetData(), PFMPoints.Num() * 3 * sizeof(float));

	UE_LOGF(LogPFMExporter, Log, "Exported PFM file %ls", *FileName);
	return true;
}

bool FPFMExporterMesh::BeginExport_RenderThread(FRHICommandListImmediate& RHICmdList)
{
	// Allocate RT+SR RHI resources
	FRHITextureCreateDesc Desc =
		FRHITextureCreateDesc::Create2D(TEXT("FPFMExporterMesh"))
		.SetExtent(DimWidth, DimHeight)
		.SetFormat(PFMTextureFormat)
		.SetInitialState(ERHIAccess::CopySrc);

	ShaderResourceTexture = RHICmdList.CreateTexture(Desc);

	Desc.SetFlags(ETextureCreateFlags::RenderTargetable | ETextureCreateFlags::ShaderResource)
		.SetInitialState(ERHIAccess::RTV);

	RenderTargetTexture = RHICmdList.CreateTexture(Desc);

	return true;
}

bool FPFMExporterMesh::FinishExport_RenderThread(FRHICommandListImmediate& RHICmdList)
{
	RHICmdList.Transition(FRHITransitionInfo(RenderTargetTexture, ERHIAccess::RTV, ERHIAccess::CopyDest));

	RHICmdList.CopyTexture(RenderTargetTexture, ShaderResourceTexture, {});
	RHICmdList.ImmediateFlush(EImmediateFlushType::FlushRHIThreadFlushResources);

	// Extract result to memory
	PFMPoints.Empty();
	PFMPoints.Reserve(DimHeight*DimWidth);
	
	uint32 SrcStride;
	uint8* SrcBuffer = (uint8*)RHICmdList.LockTexture2D(ShaderResourceTexture, 0, EResourceLockMode::RLM_ReadOnly, SrcStride, false);
	if(SrcBuffer!=nullptr)
	{
		for (int32 y = 0; y < DimHeight; ++y)
		{
			for (int32 x = 0; x < DimWidth; ++x)
			{
				float* Src = (float*)(SrcBuffer + x * PFMPixelSize + y * SrcStride);
				float Alpha = Src[3];
				if (Alpha==0)
				{					
					PFMPoints.Add(FVector(NAN, NAN, NAN));// Undefined geometry, return NAN
				}
				else
				{
					PFMPoints.Add(FVector(Src[0], Src[1], Src[2])); // Defined, save this point
				}
			}
		}
	}
	RHICmdList.UnlockTexture2D(ShaderResourceTexture, 0, false);

	// Release RHI resources
	RenderTargetTexture.SafeRelease();
	ShaderResourceTexture.SafeRelease();
	return true;
}

