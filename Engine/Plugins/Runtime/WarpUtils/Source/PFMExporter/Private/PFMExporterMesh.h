// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "RHI.h"
#include "RHICommandList.h"

class FPFMExporterMesh
{
	TArray<FVector> PFMPoints;
	int DimWidth;
	int DimHeight;
public:
	FPFMExporterMesh(int Width, int Height);
	~FPFMExporterMesh();

	bool BeginExport_RenderThread(FRHICommandListImmediate& RHICmdList);
	bool FinishExport_RenderThread(FRHICommandListImmediate& RHICmdList);
	bool SaveToFile(const FString& FileName);
	
	inline bool IsValid() const 
	{ 
		return PFMPoints.Num()>0; 
	}
	inline FIntRect GetSize() const
	{
		return FIntRect(FIntPoint(0, 0), FIntPoint(DimWidth, DimHeight));
	}

	inline FTextureRHIRef GetTargetableTexture() const
	{
		return RenderTargetTexture;
	}

private:
	FTextureRHIRef RenderTargetTexture;
	FTextureRHIRef ShaderResourceTexture;
};


