// Copyright Epic Games, Inc. All Rights Reserved.

#include "PFMExporterModule.h"
#include "PFMExporterShader.h"
#include "PFMExporterMesh.h"
#include "PFMExporterLog.h"

#include "Interfaces/IPluginManager.h"
#include "Misc/Paths.h"
#include "RenderingThread.h"
#include "ShaderCore.h"
#include "RHICommandList.h"

//////////////////////////////////////////////////////////////////////////////////////////////
// IModuleInterface
//////////////////////////////////////////////////////////////////////////////////////////////
#define WARPUTILS_SHADERS_MAP TEXT("/Plugin/WarpUtils")

void FPFMExporterModule::StartupModule()
{
	if (!AllShaderSourceDirectoryMappings().Contains(WARPUTILS_SHADERS_MAP))
	{
		FString PluginShaderDir = FPaths::Combine(IPluginManager::Get().FindPlugin(TEXT("WarpUtils"))->GetBaseDir(), TEXT("Shaders"));
		AddShaderSourceDirectoryMapping(WARPUTILS_SHADERS_MAP, PluginShaderDir);
	}
}

void FPFMExporterModule::ShutdownModule()
{	
}

//////////////////////////////////////////////////////////////////////////////////////////////
// IPFMExporter
//////////////////////////////////////////////////////////////////////////////////////////////
bool FPFMExporterModule::ExportPFM(const FStaticMeshLODResources* SrcMeshResource, const FMatrix& MeshToOrigin, int PFMWidth, int PFMHeight, const FString& FilePath)
{
	FPFMExporterMesh* PFMMesh = new FPFMExporterMesh(PFMWidth, PFMHeight);

	ENQUEUE_RENDER_COMMAND(CaptureCommand)([SrcMeshResource, MeshToOrigin, PFMMesh, FilePath](FRHICommandListImmediate& RHICmdList)
	{
		if (FPFMExporterShader::ApplyPFMExporter_RenderThread(RHICmdList, *SrcMeshResource, MeshToOrigin, *PFMMesh))
		{
			// Capture is ok, save result to file
			PFMMesh->SaveToFile(FilePath);
		}
		else
		{
			//! handle error
			UE_LOGF(LogPFMExporter, Error, "Fail to capture pfm mesh to file: %ls", *FilePath);
		}

		// Release pfm data
		delete PFMMesh;
	});
	
	return true;
}

IMPLEMENT_MODULE(FPFMExporterModule, PFMExporter);

