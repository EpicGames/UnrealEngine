// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "IPFMExporter.h"

class FPFMExporterModule 
	: public IPFMExporter
{
public:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// IModuleInterface
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

public:
	//////////////////////////////////////////////////////////////////////////////////////////////
	// PFMExporter
	//////////////////////////////////////////////////////////////////////////////////////////////
	virtual bool ExportPFM(
		const FStaticMeshLODResources* SrcMeshResource,
		const FMatrix& MeshToOrigin,
		int PFMWidth,
		int PFMHeight,
		const FString& LocalFileName
	) override;
};
