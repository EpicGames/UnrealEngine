// Copyright Epic Games, Inc. All Rights Reserved.

#include "PFMExporterShader.h"
#include "PFMExporterMesh.h"

#include "Rendering/PositionVertexBuffer.h"
#include "Shader.h"
#include "GlobalShader.h"
#include "ShaderParameters.h"
#include "ShaderParameterUtils.h"
#include "PixelShaderUtils.h"

#include "RHIResources.h"
#include "DataDrivenShaderPlatformInfo.h"
#include "CommonRenderResources.h"

#include "Engine/StaticMesh.h"
#include "HAL/IConsoleManager.h"
#include "Rendering/StaticMeshVertexBuffer.h"

#include "Components/StaticMeshComponent.h"
#include "Components/SceneComponent.h"
#include "StaticMeshResources.h"

#define PFMExporterShaderFileName TEXT("/Plugin/WarpUtils/Private/PFMExporterShaders.usf")

class FPFMExporterVS 
	: public FGlobalShader
{
	DECLARE_SHADER_TYPE(FPFMExporterVS, Global);

	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return true;
	}

	/** Default constructor. */
	FPFMExporterVS() 
	{
	}

public:
	/** Initialization constructor. */
	FPFMExporterVS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
		MeshToPFMMatrixParameter.Bind(Initializer.ParameterMap, TEXT("MeshToPFMMatrix"));
	}	

	void SetParameters(FRHIBatchedShaderParameters& BatchedParameters, const FMatrix& MeshToPFMMatrix)
	{
		SetShaderValue(BatchedParameters, MeshToPFMMatrixParameter, (FMatrix44f)MeshToPFMMatrix);
	}

private:
	LAYOUT_FIELD(FShaderParameter, MeshToPFMMatrixParameter)
};

class FPFMExporterPS 
	: public FGlobalShader
{
	DECLARE_SHADER_TYPE(FPFMExporterPS, Global);

public:
	FPFMExporterPS()
	{ 
	}

	FPFMExporterPS(const ShaderMetaType::CompiledShaderInitializerType& Initializer)
		: FGlobalShader(Initializer)
	{
	}

public:
	static bool ShouldCompilePermutation(const FGlobalShaderPermutationParameters& Parameters)
	{
		return IsFeatureLevelSupported(Parameters.Platform, ERHIFeatureLevel::SM5);
	}

	static void ModifyCompilationEnvironment(const FGlobalShaderPermutationParameters& Parameters, FShaderCompilerEnvironment& OutEnvironment)
	{
		FGlobalShader::ModifyCompilationEnvironment(Parameters, OutEnvironment);
	}	
};

// Implement shaders inside UE
IMPLEMENT_SHADER_TYPE(, FPFMExporterVS, PFMExporterShaderFileName, TEXT("PFMExporterUV_VS"), SF_Vertex);
IMPLEMENT_SHADER_TYPE(, FPFMExporterPS, PFMExporterShaderFileName, TEXT("PFMExporterPassthrough_PS"), SF_Pixel);

bool FPFMExporterShader::ApplyPFMExporter_RenderThread(
	FRHICommandListImmediate& RHICmdList,
	const FStaticMeshLODResources& SrcMeshResource,
	const FMatrix& MeshToOrigin,
	FPFMExporterMesh& DstPfmMesh
)
{
	check(IsInRenderingThread());

	//@todo: Here we can add UV channel definition (multiple PFM from one mesh with multiple materials(UV))
	int UVIndex = 0; //! Support multimaterial:

	const FPositionVertexBuffer& VertexPosition = SrcMeshResource.VertexBuffers.PositionVertexBuffer;
	const FStaticMeshVertexBuffer& VertexBuffer = SrcMeshResource.VertexBuffers.StaticMeshVertexBuffer;
	const FRawStaticIndexBuffer&   IndexBuffer  = SrcMeshResource.IndexBuffer;

	uint32 NumTriangles = IndexBuffer.GetNumIndices() / 3; //! Now by default no triangle strip supported
	uint32 NumVertices = VertexBuffer.GetNumVertices();

	// Initialize data for PFm (RT textures, etc)
	DstPfmMesh.BeginExport_RenderThread(RHICmdList);

	const FRHIBufferCreateDesc CreateDesc =
		FRHIBufferCreateDesc::CreateVertex<FFilterVertex>(TEXT("FPFMExporterShader"), NumVertices)
		.AddUsage(EBufferUsageFlags::Dynamic)
		.SetInitActionInitializer()
		.DetermineInitialState();

	TRHIBufferInitializer<FFilterVertex> Initializer = RHICmdList.CreateBufferInitializer(CreateDesc);
	{
		//Fill buffer with vertex+selected UV channel:
		for (uint32 i = 0; i < NumVertices; i++)
		{
			FFilterVertex& Vertex = Initializer[i];
			Vertex.Position = VertexPosition.VertexPosition(i);
			Vertex.UV = VertexBuffer.GetVertexUV(i, UVIndex); // Get UV from selected channel
		}
	}
	FBufferRHIRef VertexBufferRHI = Initializer.Finalize();

	FRHIBuffer* IndexBufferRHI = IndexBuffer.IndexBufferRHI;

	// Build transform matrix for mesh:
	static float Scale = 1.0f; // Default export scale is unreal, cm
	static FMatrix MPCDIToGame = FMatrix(
		FPlane(0.f, Scale, 0.f, 0.f),
		FPlane(0.f, 0.f, Scale, 0.f),
		FPlane(-Scale, 0.f, 0.f, 0.f),
		FPlane(0.f, 0.f, 0.f, 1.f));
	static FMatrix GameToMPCDI = MPCDIToGame.Inverse();
	FMatrix MeshToPFMMatrix = MeshToOrigin * GameToMPCDI;

	{// Do remap single render pass		
		FRHIRenderPassInfo RPInfo(DstPfmMesh.GetTargetableTexture(), ERenderTargetActions::Load_Store);
		RHICmdList.BeginRenderPass(RPInfo, TEXT("DisplayClusterPFMExporterShader"));
		{
			FIntRect DstRect = DstPfmMesh.GetSize();
			RHICmdList.SetViewport(DstRect.Min.X, DstRect.Min.Y, 0.0f, DstRect.Max.X, DstRect.Max.Y, 1.0f);
			//DrawClearQuad(RHICmdList, FLinearColor::Black); //!

			FGlobalShaderMap* ShaderMap = GetGlobalShaderMap(GMaxRHIFeatureLevel);
			TShaderMapRef<FPFMExporterVS> VertexShader(ShaderMap);
			TShaderMapRef<FPFMExporterPS> PixelShader(ShaderMap);

			{// Set the graphic pipeline state.			
				FGraphicsPipelineStateInitializer GraphicsPSOInit;
				RHICmdList.ApplyCachedRenderTargets(GraphicsPSOInit);

				GraphicsPSOInit.DepthStencilState = TStaticDepthStencilState<false, CF_Never>::GetRHI();
				GraphicsPSOInit.BlendState = TStaticBlendState <>::GetRHI();
				GraphicsPSOInit.RasterizerState = TStaticRasterizerState<>::GetRHI();

				GraphicsPSOInit.BoundShaderState.VertexDeclarationRHI = GFilterVertexDeclaration.VertexDeclarationRHI;// GetVertexDeclarationFVector4();
				GraphicsPSOInit.BoundShaderState.VertexShaderRHI = VertexShader.GetVertexShader();
				GraphicsPSOInit.BoundShaderState.PixelShaderRHI = PixelShader.GetPixelShader();
				GraphicsPSOInit.PrimitiveType = PT_TriangleList;

				SetGraphicsPipelineState(RHICmdList, GraphicsPSOInit, 0);
			}

			SetShaderParametersLegacyVS(RHICmdList, VertexShader, MeshToPFMMatrix);

			// Render mesh to PFM texture:
			RHICmdList.SetStreamSource(0, VertexBufferRHI, 0);
			RHICmdList.DrawIndexedPrimitive(IndexBufferRHI, 0, 0, NumVertices, 0, NumTriangles, 1);
		}
		RHICmdList.EndRenderPass();
	}
	
	// Extract data from texture to memory:
	return DstPfmMesh.FinishExport_RenderThread(RHICmdList);
}


