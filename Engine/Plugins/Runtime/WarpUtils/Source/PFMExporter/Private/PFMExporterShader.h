// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "RHI.h"
#include "RHICommandList.h"
#include "RenderResource.h"

#include "IPFMExporter.h"

class FPFMExporterMesh;

class FPFMExporterShader
{
public:
	static bool ApplyPFMExporter_RenderThread(
		FRHICommandListImmediate& RHICmdList, 
		const FStaticMeshLODResources& SrcMeshResource,
		const FMatrix& MeshToOrigin,
		FPFMExporterMesh& DstPfmMesh
	);
};