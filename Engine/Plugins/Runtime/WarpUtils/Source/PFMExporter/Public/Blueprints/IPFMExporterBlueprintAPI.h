// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "UObject/Interface.h"
#include "RHI.h"
#include "RHIResources.h"

#include "IPFMExporterBlueprintAPI.generated.h"

class UStaticMeshComponent;
class USceneComponent;

UINTERFACE(meta = (CannotImplementInterfaceInBlueprint))
class UPFMExporterBlueprintAPI : public UInterface
{
	GENERATED_BODY()
};

class IPFMExporterBlueprintAPI
{
	GENERATED_BODY()

public:
	/**
	* Generate PFM file from static mesh.
	* The UV channel must be defined, assigned range 0..1 used as screen surface.
	* Origin assigned by function arg, or by default used mesh parent.
	*
	* @param SrcMesh - Static mesh with assigned UV channel, used as export source of PFM file
	* @param Origin - (Optional) Custom cave origin node, if not defined, used SrcMesh parent
	* @param Width - Output PFM mesh texture width
	* @param Height - Output PFM mesh texture height
	* @param FileName - Output PFM file name
	*
	* @return true, if success
	*/
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Export Static Mesh to PFM file"), Category = "PFMExporter")
	virtual bool ExportPFM(
		UStaticMeshComponent*     SrcMesh,
		USceneComponent* Origin,
		int Width,
		int Height,
		const FString& FileName
	) = 0;
};
