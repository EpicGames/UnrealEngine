// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "Blueprints/IPFMExporterBlueprintAPI.h"
#include "PFMExporterBlueprintLib.generated.h"

/**
 * Blueprint API function library
 */
UCLASS()
class UPFMExporterBlueprintLib
	: public UBlueprintFunctionLibrary
{
	GENERATED_UCLASS_BODY()

public:
	/** Return Display Cluster API interface. */
	UFUNCTION(BlueprintPure, meta = (DisplayName = "PFMExporter Module API"), Category = "nDisplay")
	static void GetAPI(TScriptInterface<IPFMExporterBlueprintAPI>& OutAPI);
};
