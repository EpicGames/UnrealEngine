// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

struct FStaticMeshLODResources;

class IPFMExporter : public IModuleInterface
{
public:
	static constexpr auto ModuleName = TEXT("PFMExporter");
public:
	/**
	 * Singleton-like access to this module's interface.  This is just for convenience!
	 * Beware of calling this during the shutdown phase, though.  Your module might have been unloaded already.
	 *
	 * @return Returns singleton instance, loading the module on demand if needed
	 */
	static inline IPFMExporter& Get()
	{
		return FModuleManager::LoadModuleChecked<IPFMExporter>(IPFMExporter::ModuleName);
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded(IPFMExporter::ModuleName);
	}

public:
	/**
	* Generate PFM file from static mesh. 
	* The UV channel must be defined, assigned range 0..1 used as screen surface. 	
	* 
 	* @param SrcMeshResource - Source mesh with defined UV, used as PFM 3d source
	* @param MeshToOrigin - Transform matrix convert mesh vertices to cave origin space
	* @param PFMWidth - Output PFM mesh texture width
	* @param PFMHeight - Output PFM mesh texture height
	* @param LocalFileName - Output PFM file name (supported relative paths)
	*
	* @return - true, if export success
	*/
	virtual bool ExportPFM(
		const FStaticMeshLODResources* SrcMeshResource,
		const FMatrix& MeshToOrigin,
		int PFMWidth,
		int PFMHeight,
		const FString& LocalFileName
	) = 0;

};
