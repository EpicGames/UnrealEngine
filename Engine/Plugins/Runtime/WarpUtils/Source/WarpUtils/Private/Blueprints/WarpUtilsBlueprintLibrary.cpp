// Copyright Epic Games, Inc. All Rights Reserved.

#include "Blueprints/WarpUtilsBlueprintLibrary.h"

#include "GameFramework/Actor.h"
#include "HAL/FileManager.h"

#include "WarpUtilsLog.h"


bool UWarpUtilsBlueprintLibrary::SavePFM(const FString& File, const int TexWidth, const int TexHeight, const TArray<FVector>& Vertices)
{
	if (Vertices.Num() != (TexWidth * TexHeight))
	{
		UE_LOGF(LogWarpUtilsBlueprint, Error, "Wrong pixels amount: TexWidth %d, TexHeight %d, TexWidth*TexHeight == %d, Vertices %d",
			TexWidth, TexHeight, TexWidth * TexHeight, Vertices.Num());

		return false;
	}

	const size_t ArraySize = 3 * Vertices.Num();
	TArray<float> Dataset;
	Dataset.AddUninitialized(ArraySize);

	float Scale = 1.0f;
	FMatrix m = FMatrix(
		FPlane(0.f, Scale, 0.f, 0.f),
		FPlane(0.f, 0.f, Scale, 0.f),
		FPlane(-Scale, 0.f, 0.f, 0.f),
		FPlane(0.f, 0.f, 0.f, 1.f));

	int len = Vertices.Num();
	for (int i = 0; i < len; i++)
	{
		FVector v = Vertices[i];

		if (!FMath::IsNaN(v.X))
		{
			v = m.InverseTransformPosition(v);
		}

		Dataset[i * 3 + 0] = v.X;
		Dataset[i * 3 + 1] = v.Y;
		Dataset[i * 3 + 2] = v.Z;
	}

	TUniquePtr<FArchive> FileWriter(IFileManager::Get().CreateFileWriter(*File));
	if (!FileWriter.IsValid())
	{
		UE_LOGF(LogWarpUtilsBlueprint, Error, "Couldn't create a file writer %ls", *File);
		return false;
	}

	const FString Header = FString::Printf(TEXT("PF%c%d %d%c-1%c"), 0x0A, TexWidth, TexHeight, 0x0A, 0x0A);
	FileWriter->Serialize(TCHAR_TO_ANSI(*Header), Header.Len());
	FileWriter->Serialize((void*)Dataset.GetData(), ArraySize * sizeof(float));

	return true;
}

bool UWarpUtilsBlueprintLibrary::SavePFMEx(const FString& File, const int TexWidth, const int TexHeight, const TArray<FVector>& Vertices, const TArray<bool>& VertexValidityFlags)
{
	if (Vertices.Num() != (TexWidth * TexHeight))
	{
		UE_LOGF(LogWarpUtilsBlueprint, Error, "Wrong pixels amount: TexWidth %d, TexHeight %d, TexWidth*TexHeight == %d, Vertices %d",
			TexWidth, TexHeight, TexWidth * TexHeight, Vertices.Num());

		return false;
	}

	if (Vertices.Num() != VertexValidityFlags.Num())
	{
		UE_LOGF(LogWarpUtilsBlueprint, Error, "Vertices amount not equals to the validity flags amount (%d != %d)", Vertices.Num(), VertexValidityFlags.Num());
		return false;
	}

	// Explicit XYZ set to NAN instead of FVector(NAN, NAN, NAN) allows to avoid any FVector internal NAN checks
	FVector NanVector;
	NanVector.X = NanVector.Y = NanVector.Z = NAN;

	// Replace invalid vertices with NaN values
	TArray<FVector> FixedData(Vertices);
	for (int i = 0; i < FixedData.Num(); ++i)
	{
		FixedData[i] = (VertexValidityFlags[i] ? FixedData[i] : NanVector);
	}

	// Save data
	return SavePFM(File, TexWidth, TexHeight, FixedData);
}

bool UWarpUtilsBlueprintLibrary::GeneratePFM(
	const FString& File,
	const FVector& StartLocation, const FRotator& StartRotation, const AActor* PFMOrigin,
	const int TilesHorizontal, const int TilesVertical, const float ColumnAngle,
	const float TileSizeHorizontal, const float TileSizeVertical, const int TilePixelsHorizontal, const int TilePixelsVertical, const bool AddMargin)
{
	// Check input data that is used in this function
	if (TilesHorizontal < 1 || TilesVertical < 1)
	{
		UE_LOGF(LogWarpUtilsBlueprint, Error, "Both horizontal and vertical tiles amounts must be more than 1");
		return false;
	}

	// Amount of validity flags must be the same as the tiles amount
	const int TilesAmount = TilesHorizontal * TilesVertical;

	// Set all validity flags to 'true' by default
	TArray<bool> TilesValidityFlags;
	TilesValidityFlags.AddUninitialized(TilesAmount);
	for (bool& Item : TilesValidityFlags)
	{
		Item = true;
	}

	// Call the implementation function
	return GeneratePFMEx(File, StartLocation, StartRotation, PFMOrigin, TilesHorizontal, TilesVertical, ColumnAngle, TileSizeHorizontal, TileSizeVertical, TilePixelsHorizontal, TilePixelsVertical, AddMargin, TilesValidityFlags);
}

bool UWarpUtilsBlueprintLibrary::GeneratePFMEx(
	const FString& File,
	const FVector& StartLocation, const FRotator& StartRotation, const AActor* PFMOrigin,
	const int TilesHorizontal, const int TilesVertical, const float ColumnAngle,
	const float TileSizeHorizontal, const float TileSizeVertical, const int TilePixelsHorizontal, const int TilePixelsVertical, const bool AddMargin, const TArray<bool>& TilesValidityFlags)
{
	// Check all input data
	if (File.IsEmpty() || !PFMOrigin || TilesHorizontal < 1 || TilesVertical < 1 || TileSizeHorizontal < 1.f || TileSizeVertical < 1.f || TilePixelsHorizontal < 2 || TilePixelsVertical < 2 || TilesValidityFlags.Num() != TilesHorizontal * TilesVertical)
	{
		UE_LOGF(LogWarpUtilsBlueprint, Error, "Wrong input data");
		return false;
	}

	// Explicit XYZ set to NAN instead of FVector(NAN, NAN, NAN) allows to avoid any FVector internal NAN checks
	FVector NanVector;
	NanVector.X = NanVector.Y = NanVector.Z = NAN;

	// Texture XY size
	const int TexSizeX = TilesHorizontal * TilePixelsHorizontal;
	const int TexSizeY = TilesVertical * TilePixelsVertical;
	
	// Prepare array for output data
	TArray<FVector> PFMData;
	PFMData.Reserve(TexSizeX * TexSizeY);

	// Some constants for navigation math
	const FTransform StartFrom = FTransform(StartRotation, StartLocation);
	const FVector RowTranslate(0.f, 0.f, -TileSizeVertical);
	const FVector ColTranslate(0.f, TileSizeHorizontal, 0.f);

	// Compute horizontal margins and pixel offsets
	const float PixelOffsetX = (AddMargin ? TileSizeHorizontal / TilePixelsHorizontal : TileSizeHorizontal / (TilePixelsHorizontal - 1));
	const float MarginX = (AddMargin ? PixelOffsetX / 2.f : 0.f);
	// Compute vertical margins and pixel offsets
	const float PixelOffsetY = (AddMargin ? TileSizeVertical / TilePixelsVertical : TileSizeVertical / (TilePixelsVertical - 1));
	const float MarginY = (AddMargin ? PixelOffsetY / 2.f : 0.f);

	// Cache tile transforms so we won't have to compute it for every pixel
	TMap<int, TMap<int, FTransform>> TransformsCache;

	// The order is from left to right, from top to bottom
	for (int Y = 0; Y < TexSizeY; ++Y)
	{
		// Current row index
		const int CurRow = Y / TilePixelsVertical;
		// Transform for current row
		const FTransform CurRowTransform = (CurRow == 0 ? StartFrom : FTransform(TransformsCache[CurRow - 1][0].Rotator(), TransformsCache[CurRow - 1][0].TransformPosition(RowTranslate)));

		// Cache current row transform
		TransformsCache.Emplace(CurRow);
		TransformsCache[CurRow].Emplace(0, CurRowTransform);

		for (int X = 0; X < TexSizeX; ++X)
		{
			// Current column index
			const int CurCol = X / TilePixelsHorizontal;
			// Check if the column transform has been cached previously
			const bool bCached = TransformsCache[CurRow].Contains(CurCol);
			// Transform of the current tile (top left corner)
			const FTransform CurTileTransform = (bCached ? 
				TransformsCache[CurRow][CurCol] : 
				(CurCol == 0 ? CurRowTransform : FTransform(TransformsCache[CurRow][CurCol - 1].Rotator().Add(0.f, ColumnAngle, 0.f), TransformsCache[CurRow][CurCol - 1].TransformPosition(ColTranslate))));

			// Cache new transform
			if (!bCached)
			{
				TransformsCache[CurRow].Emplace(CurCol, CurTileTransform);
			}

			// XY within a tile
			const int TilePixelY = Y % TilePixelsVertical;
			const int TilePixelX = X % TilePixelsHorizontal;

			// Tile index in the ValidityFlags array
			const int TileArrayIdx = CurCol * TilesVertical + CurRow;

			// Fake tiles produce Nan values
			if (TilesValidityFlags[TileArrayIdx] == false)
			{
				PFMData.Add(NanVector);
				continue;
			}

			// Pixel offset in tile space
			const FVector TileSpaceOffset = FVector(0.f, MarginX + TilePixelX * PixelOffsetX, -(MarginY + TilePixelY * PixelOffsetY));
			// Pixel in world space
			const FVector WorldSpacePixel = CurTileTransform.TransformPosition(TileSpaceOffset);
			// Pixel in the PFM origin space
			const FVector PFMSpacePixel = PFMOrigin->GetActorTransform().InverseTransformPosition(WorldSpacePixel);

			// Store current pixel data
			PFMData.Add(PFMSpacePixel);
		}
	}

	// Save generated data to file
	return SavePFM(File, TexSizeX, TexSizeY, PFMData);
}
