// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "WarpUtilsBlueprintLibrary.generated.h"


UCLASS()
class UWarpUtilsBlueprintLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:
	// Save data to a PFM file
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Save PFM"), Category = "Miscellaneous|Warp")
	static bool SavePFM(const FString& File, const int TexWidth, const int TexHeight, const TArray<FVector>& Vertices);

	// Save data to a PFM file. Since the float NaN value is not available in blueprints, we provide a flags array (false == NaN)
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Save PFM Extended"), Category = "Miscellaneous|Warp")
	static bool SavePFMEx(const FString& File, const int TexWidth, const int TexHeight, const TArray<FVector>& Vertices, const TArray<bool>& TilesValidityFlags);

	// Generate and save data to a PFM file
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Generate PFM"), Category = "Miscellaneous|Warp")
	static bool GeneratePFM(
		const FString& File,
		const FVector& StartLocation, const FRotator& StartRotation, const AActor* PFMOrigin,
		const int TilesHorizontal, const int TilesVertical, const float ColumnAngle,
		const float TileSizeHorizontal, const float TileSizeVertical, const int TilePixelsHorizontal, const int TilePixelsVertical, const bool AddMargin
	);

	// Generate and save data to a PFM file. Additionally, we have an array of tiles validiy flags (false == all pixels of a tile are NaN)
	UFUNCTION(BlueprintCallable, meta = (DisplayName = "Generate PFM Extended"), Category = "Miscellaneous|Warp")
	static bool GeneratePFMEx(
		const FString& File,
		const FVector& StartLocation, const FRotator& StartRotation, const AActor* PFMOrigin,
		const int TilesHorizontal, const int TilesVertical, const float ColumnAngle,
		const float TileSizeHorizontal, const float TileSizeVertical, const int TilePixelsHorizontal, const int TilePixelsVertical, const bool AddMargin,
		const TArray<bool>& TilesValidityFlags
	);
};
