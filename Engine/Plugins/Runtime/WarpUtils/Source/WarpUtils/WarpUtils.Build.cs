// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class WarpUtils : ModuleRules
{
	public WarpUtils(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(
			new string[]
			{
				"Core",
			});
		
		PrivateDependencyModuleNames.AddRange(
			new string[]
			{
				"CoreUObject",
				"Engine",
				"Slate",
				"SlateCore",
			});
	}
}
