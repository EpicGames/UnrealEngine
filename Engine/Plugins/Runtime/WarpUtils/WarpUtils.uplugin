{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "Warp Utils",
	"Description": "PFM/MPCDI generation & visualization",
	"Category": "Misc",
	"CreatedBy": "Epic Games, Inc.",
	"CreatedByURL": "https://epicgames.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": true,
	"IsBetaVersion": true,
	"Installed": false,
	"Modules": [
		{
			"Name": "PFMExporter",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [
				"Win64"
			]
		},
		{
			"Name": "WarpUtils",
			"Type": "Runtime",
			"LoadingPhase": "Default"
		}
	]
}
